[Plugins]
+ScriptSupportedModules=CoreUObject
+ScriptSupportedModules=Engine
+ScriptSupportedModules=ScriptPlugin

[/Script/Engine.GarbageCollectionSettings]
gc.MaxObjectsInProgram=262144

[UnrealHeaderTool]
+StructsWithNoPrefix="uint64"
+StructsWithNoPrefix="uint32"
+StructsWithNoPrefix="double"
+StructsWithTPrefix="IndirectArray"
+StructsWithTPrefix="BitArray"
+StructsWithTPrefix="SparseArray"
+StructsWithTPrefix="Set"
+StructsWithTPrefix="Map"
+StructsWithTPrefix="MultiMap"
+StructsWithTPrefix="SharedPtr"
+DelegateParameterCountStrings="_OneParam"
+DelegateParameterCountStrings="_TwoParams"
+DelegateParameterCountStrings="_ThreeParams"
+DelegateParameterCountStrings="_FourParams"
+DelegateParameterCountStrings="_FiveParams"
+DelegateParameterCountStrings="_SixParams"
+DelegateParameterCountStrings="_SevenParams"
+DelegateParameterCountStrings="_EightParams"
+DelegateParameterCountStrings="_NineParams"
+TypeRedirects=(OldType="FStringAssetReference", NewType="FSoftObjectPath")
+TypeRedirects=(OldType="FStringClassReference", NewType="FSoftClassPath")
+TypeRedirects=(OldType="TAssetPtr", NewType="TSoftObjectPtr")
+TypeRedirects=(OldType="TAssetSubclassOf", NewType="TSoftClassPtr")
+Units="Micrometers"
+Units="um"
+Units="\u00B5m"
+Units="Millimeters"
+Units="mm"
+Units="Centimeters"
+Units="cm"
+Units="Meters"
+Units="m"
+Units="Kilometers"
+Units="km"
+Units="Inches"
+Units="in"
+Units="Feet"
+Units="ft"
+Units="Yards"
+Units="yd"
+Units="Miles"
+Units="mi"
+Units="Lightyears"
+Units="ly"
+Units="Degrees"
+Units="deg"
+Units="\u00B0"
+Units="Radians"
+Units="rad"
+Units="CentimetersPerSecond"
+Units="cm/s"
+Units="MetersPerSecond"
+Units="m/s"
+Units="KilometersPerHour"
+Units="km/h"
+Units="kmph"
+Units="MilesPerHour"
+Units="mi/h"
+Units="mph"
+Units="DegreesPerSecond"
+Units="deg/s"
+Units="RadiansPerSecond"
+Units="rad/s"
+Units="CentimetersPerSecondSquared"
+Units="cm/s\u00B2"
+Units="cm/s^2"
+Units="MetersPerSecondSquared"
+Units="m/s\u00B2"
+Units="m/s^2"
+Units="Celsius"
+Units="C"
+Units="degC"
+Units="\u00B0C"
+Units="Farenheit"
+Units="F"
+Units="degF"
+Units="\u00B0F"
+Units="Kelvin"
+Units="K"
+Units="Micrograms"
+Units="ug")
+Units="\u00B5g"
+Units="Milligrams"
+Units="mg"
+Units="Grams"
+Units="g"
+Units="Kilograms"
+Units="kg"
+Units="MetricTons"
+Units="t"
+Units="Ounces"
+Units="oz"
+Units="Pounds"
+Units="lb"
+Units="Stones"
+Units="st"
+Units="Newtons"
+Units="N"
+Units="PoundsForce"
+Units="lbf"
+Units="KilogramsForce"
+Units="kgf"
+Units="KilogramsCentimetersPerSecondSquared"
+Units="kgcm/s\u00B2"
+Units="NewtonMeters"
+Units="Nm"
+Units="kgcm"
+Units="KilogramCentimetersSquaredPerSecondSquared"
+Units="Decibels"
+Units="dB"
+Units="kgcm\u00B2/s\u00B2"
+Units="Hertz"
+Units="Hz"
+Units="Kilohertz"
+Units="KHz"
+Units="Megahertz"
+Units="MHz"
+Units="Gigahertz"
+Units="GHz"
+Units="RevolutionsPerMinute"
+Units="rpm"
+Units="Bytes"
+Units="B"
+Units="Kilobytes"
+Units="KB"
+Units="KiB"
+Units="Megabytes"
+Units="MB"
+Units="MiB"
+Units="Gigabytes"
+Units="GB"
+Units="GiB"
+Units="Terabytes"
+Units="TB"
+Units="TiB"
+Units="Lumens"
+Units="lm"
+Units="Candela"
+Units="cd"
+Units="Lux"
+Units="lx"
+Units="CandelaPerMeterSquared"
+Units="cd/m2"
+Units="CandelaPerMeter2"
+Units="EV"
+Units="Nits"
+Units="nt"
+Units="Nanoseconds"
+Units="ns"
+Units="Microseconds"
+Units="us"
+Units="\u00B5s"
+Units="Milliseconds"
+Units="ms"
+Units="Seconds"
+Units="s"
+Units="Minutes"
+Units="min"
+Units="Hours"
+Units="hrs"
+Units="Days"
+Units="dy"
+Units="Months"
+Units="mth"
+Units="Years"
+Units="yr"
+Units="ppi"
+Units="dpi"
+Units="Percent"
+Units="%"
+Units="times"
+Units="x"
+Units="multiplier"

[UnrealHeaderTool]
EngineNativePointerMemberBehavior=Disallow
EngineObjectPtrMemberBehavior=AllowSilently
EnginePluginNativePointerMemberBehavior=AllowSilently
EnginePluginObjectPtrMemberBehavior=AllowSilently
NonEngineNativePointerMemberBehavior=AllowSilently
NonEngineObjectPtrMemberBehavior=AllowSilently
AutomaticSettersAndGetters=False
AreRigVMUObjectPropertiesEnabled=False
AreRigVMUInterfacePropertiesEnabled=True
ShowDeprecations=True

[UnrealHeaderTool]
+DocumentationPolicies=(Name="Strict", ClassOrStructCommentRequired=true, FunctionToolTipsRequired=true, MemberToolTipsRequired=true, ParameterToolTipsRequired=true, FloatRangesRequired=true)
DefaultDocumentationPolicy=""
